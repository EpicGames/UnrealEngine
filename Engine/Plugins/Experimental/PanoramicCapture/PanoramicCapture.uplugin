{
	"FileVersion": 3,
	"Version": 1.1,
	"VersionName": "Alpha",
	"FriendlyName": "Panoramic Capture",
	"Description": "A plugin to capture a sequence of panoramic images in monoscopic or stereoscopic (top/bottom).",
	"Category": "Movie Capture",
	"CreatedBy": "Kite & Lightning",
	"CreatedByURL": "https://kiteandlightning.la/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PanoramicCapture",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}