// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PanoramicCapture : ModuleRules
	{
		public PanoramicCapture( ReadOnlyTargetRules Target ) : base(Target)
		{
			PrivateIncludePaths.AddRange(
				new string[] {
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"ImageWrapper",
					"InputCore",
					"RenderCore",
					"RHI",
					"Slate",
					"MessageLog",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
