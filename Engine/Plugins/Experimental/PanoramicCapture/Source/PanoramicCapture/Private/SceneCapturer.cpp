// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneCapturer.h"
#include "StereoPanoramaManager.h"
#include "StereoPanorama.h"
#include "StereoCapturePawn.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "UnrealEngine.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "GameFramework/Actor.h"
#include "GameFramework/DefaultPawn.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "TextureResource.h"
#include "TickableObjectRenderThread.h"
#include "Engine/BlendableInterface.h"
#include "ImageUtils.h"
#include "CoreMinimal.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineUtils.h"
#include "CoreMinimal.h"
#include "LatentActions.h"
#include "Engine/LatentActionManager.h"
#include "MessageLogModule.h"
#include "Tickable.h"
#include "RenderingThread.h"

#define LOCTEXT_NAMESPACE "LogStereoPanorama"

const FName StereoPanoramaLogName("LogStereoPanorama");

// always combine both eyes
const bool CombineAtlasesOnOutput = true;

// Rotated Grid Supersampling
const int32 maxNumSamples = 16;
struct SamplingPattern
{
    int numSamples;
    FVector2D ssOffsets[maxNumSamples];
};
const SamplingPattern g_ssPatterns[] =
{
    {
        1,
        {
            FVector2D(0, 0),
        }
    },
    {
        4,
        {
            FVector2D(0.125f, 0.625f),
            FVector2D(0.375f, 0.125f),
            FVector2D(0.625f, 0.875f),
            FVector2D(0.875f, 0.375f),
        }
    },
    {
        16,
        {
            FVector2D(0.125f, 0.125f),
            FVector2D(0.125f, 0.375f),
            FVector2D(0.125f, 0.625f),
            FVector2D(0.125f, 0.875f),
            FVector2D(0.375f, 0.125f),
            FVector2D(0.375f, 0.375f),
            FVector2D(0.375f, 0.625f),
            FVector2D(0.375f, 0.875f),
            FVector2D(0.625f, 0.125f),
            FVector2D(0.625f, 0.375f),
            FVector2D(0.625f, 0.625f),
            FVector2D(0.625f, 0.875f),
            FVector2D(0.875f, 0.125f),
            FVector2D(0.875f, 0.375f),
            FVector2D(0.875f, 0.625f),
            FVector2D(0.875f, 0.875f),
        }
    },

};

void USceneCapturer::InitCaptureComponent(USceneCaptureComponent2D* CaptureComponent, float HFov, float VFov)
{
	CaptureComponent->SetVisibility( true );
	CaptureComponent->SetHiddenInGame( false );
	CaptureComponent->FOVAngle = FMath::Max( HFov, VFov );
	CaptureComponent->bCaptureEveryFrame = false;
	CaptureComponent->bCaptureOnMovement = false;
	CaptureComponent->CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;
	DisableUnsupportedPostProcesses(CaptureComponent);

	const FName TargetName = MakeUniqueObjectName(this, UTextureRenderTarget2D::StaticClass(), TEXT("SceneCaptureTextureTarget"));
	CaptureComponent->TextureTarget = NewObject<UTextureRenderTarget2D>(this, TargetName);
	CaptureComponent->TextureTarget->InitCustomFormat(CaptureWidth, CaptureHeight, PF_FloatRGBA, false);
	CaptureComponent->TextureTarget->ClearColor = FLinearColor::Red;
	CaptureComponent->TextureTarget->TargetGamma = 2.2f;
	CaptureComponent->RegisterComponentWithWorld( GetWorld() ); //GWorld

	// Unreal Engine cannot serialize an array of subobject pointers, so add these objects to the root
	CaptureComponent->AddToRoot();
}

USceneCapturer::USceneCapturer(FVTableHelper& Helper)
    : Super(Helper)
    , ImageWrapperModule(FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper")))
    , bIsTicking(false)
    , CapturePlayerController(NULL)
    , CaptureGameMode(NULL)
    , hAngIncrement(FStereoPanoramaManager::HorizontalAngularIncrement->GetFloat())
    , vAngIncrement(FStereoPanoramaManager::VerticalAngularIncrement->GetFloat())
    , eyeSeparation(FStereoPanoramaManager::EyeSeparation->GetFloat())
    , NumberOfHorizontalSteps((int32)(360.0f / hAngIncrement))
    , NumberOfVerticalSteps((int32)(180.0f / vAngIncrement) + 1) /* Need an extra b/c we only grab half of the top & bottom slices */
    , SphericalAtlasWidth(FStereoPanoramaManager::StepCaptureWidth->GetInt())
    , SphericalAtlasHeight(SphericalAtlasWidth / 2)
    , bForceAlpha(FStereoPanoramaManager::ForceAlpha->GetInt() != 0)
    , bEnableBilerp(FStereoPanoramaManager::EnableBilerp->GetInt() != 0)
    , SSMethod(FMath::Clamp<int32>(FStereoPanoramaManager::SuperSamplingMethod->GetInt(), 0, UE_ARRAY_COUNT(g_ssPatterns)))
    , bOverrideInitialYaw(FStereoPanoramaManager::ShouldOverrideInitialYaw->GetInt() != 0)
    , ForcedInitialYaw(FRotator::ClampAxis(FStereoPanoramaManager::ForcedInitialYaw->GetFloat()))
    , OutputDir(FStereoPanoramaManager::OutputDir->GetString().IsEmpty() ? FPaths::ProjectSavedDir() / TEXT("StereoPanorama") : FStereoPanoramaManager::OutputDir->GetString())
	, UseCameraRotation(FStereoPanoramaManager::UseCameraRotation->GetInt())
    , dbgDisableOffsetRotation(FStereoPanoramaManager::FadeStereoToZeroAtSides->GetInt() != 0)
	, OutputBitDepth(FStereoPanoramaManager::OutputBitDepth->GetInt())
	, bOutputSceneDepth(FStereoPanoramaManager::OutputSceneDepth->GetInt() != 0)
	, bOutputFinalColor(FStereoPanoramaManager::OutputFinalColor->GetInt() != 0)
	, bOutputWorldNormal(FStereoPanoramaManager::OutputWorldNormal->GetInt() != 0)
	, bOutputRoughness(FStereoPanoramaManager::OutputRoughness->GetInt() != 0)
	, bOutputMetallic(FStereoPanoramaManager::OutputMetalic->GetInt() != 0)
	, bOutputBaseColor(FStereoPanoramaManager::OutputBaseColor->GetInt() != 0)
	, bOutputAmbientOcclusion(FStereoPanoramaManager::OutputAmbientOcclusion->GetInt() != 0)
	, bMonoscopicMode(FStereoPanoramaManager::MonoscopicMode->GetInt() != 0)
{}

USceneCapturer::USceneCapturer()
	: ImageWrapperModule(FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper")))
	, bIsTicking(false)
	, CapturePlayerController(NULL)
	, CaptureGameMode(NULL)
	, hAngIncrement(FStereoPanoramaManager::HorizontalAngularIncrement->GetFloat())
	, vAngIncrement(FStereoPanoramaManager::VerticalAngularIncrement->GetFloat())
	, eyeSeparation(FStereoPanoramaManager::EyeSeparation->GetFloat())
	, NumberOfHorizontalSteps((int32)(360.0f / hAngIncrement))
	, NumberOfVerticalSteps((int32)(180.0f / vAngIncrement) + 1) /* Need an extra b/c we only grab half of the top & bottom slices */
	, SphericalAtlasWidth(FStereoPanoramaManager::StepCaptureWidth->GetInt())
	, SphericalAtlasHeight(SphericalAtlasWidth / 2)
	, bForceAlpha(FStereoPanoramaManager::ForceAlpha->GetInt() != 0)
	, bEnableBilerp(FStereoPanoramaManager::EnableBilerp->GetInt() != 0)
	, SSMethod(FMath::Clamp<int32>(FStereoPanoramaManager::SuperSamplingMethod->GetInt(), 0, UE_ARRAY_COUNT(g_ssPatterns)))
	, bOverrideInitialYaw(FStereoPanoramaManager::ShouldOverrideInitialYaw->GetInt() != 0)
	, ForcedInitialYaw(FRotator::ClampAxis(FStereoPanoramaManager::ForcedInitialYaw->GetFloat()))
	, OutputDir(FStereoPanoramaManager::OutputDir->GetString().IsEmpty() ? FPaths::ProjectSavedDir() / TEXT("StereoPanorama") : FStereoPanoramaManager::OutputDir->GetString())
	, UseCameraRotation(FStereoPanoramaManager::UseCameraRotation->GetInt())
	, dbgDisableOffsetRotation(FStereoPanoramaManager::FadeStereoToZeroAtSides->GetInt() != 0)
	, OutputBitDepth(FStereoPanoramaManager::OutputBitDepth->GetInt())
	, bOutputSceneDepth(FStereoPanoramaManager::OutputSceneDepth->GetInt() != 0)
	, bOutputFinalColor(FStereoPanoramaManager::OutputFinalColor->GetInt() != 0)
	, bOutputWorldNormal(FStereoPanoramaManager::OutputWorldNormal->GetInt() != 0)
	, bOutputRoughness(FStereoPanoramaManager::OutputRoughness->GetInt() != 0)
	, bOutputMetallic(FStereoPanoramaManager::OutputMetalic->GetInt() != 0)
	, bOutputBaseColor(FStereoPanoramaManager::OutputBaseColor->GetInt() != 0)
	, bOutputAmbientOcclusion(FStereoPanoramaManager::OutputAmbientOcclusion->GetInt() != 0)
	, bMonoscopicMode(FStereoPanoramaManager::MonoscopicMode->GetInt() != 0)
{
	if (HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		return;
	}

	//NOTE: ikrimae: Keeping the old sampling mechanism just until we're sure the new way is always better
	dbgMatchCaptureSliceFovToAtlasSliceFov = false;

	// NOTE: fmaheux: Adding support for monoscopic capture: lefteye containers will be used to store data
	//				  righteye containers will be initialized but wont be processed
	// remove eye separation to avoid offset in mono
	if (bMonoscopicMode)
	{
		eyeSeparation = 0.0f;
	}

	// Add a message log category for this plugin
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions MessageLogOptions;
	MessageLogOptions.bShowPages = true;
	MessageLogOptions.bAllowClear = true;
	MessageLogOptions.MaxPageCount = 10;
	MessageLogModule.RegisterLogListing(StereoPanoramaLogName, LOCTEXT("StereoPanoramaLogLabel", "Panoramic Capture Log"));
	
	// Get all blendable materials
	ConstructorHelpers::FObjectFinder<UMaterial> Tmp1(TEXT("/PanoramicCapture/Materials/WorldNormal.WorldNormal"));
	MaterialBlendableWorldNormal = Tmp1.Object;
	
	ConstructorHelpers::FObjectFinder<UMaterial> Tmp2(TEXT("/PanoramicCapture/Materials/AmbientOcclusion.AmbientOcclusion"));
	MaterialBlendableAO = Tmp2.Object;
	
	ConstructorHelpers::FObjectFinder<UMaterial> Tmp3(TEXT("/PanoramicCapture/Materials/BaseColor.BaseColor"));
	MaterialBlendableBaseColor = Tmp3.Object;
	
	ConstructorHelpers::FObjectFinder<UMaterial> Tmp4(TEXT("/PanoramicCapture/Materials/Metallic.Metallic"));
	MaterialBlendableMetallic = Tmp4.Object;
	
	ConstructorHelpers::FObjectFinder<UMaterial> Tmp5(TEXT("/PanoramicCapture/Materials/Roughness.Roughness"));
	MaterialBlendableRoughness = Tmp5.Object;
	
	// Cache all PP volumes and current state
	CacheAllPostProcessVolumes();

    float captureHFov = 0, captureVFov = 0;

    if (dbgMatchCaptureSliceFovToAtlasSliceFov)
    {
        
		// check bitdepth
		ensure(OutputBitDepth == 8 || OutputBitDepth == 32);
		
		
		//Slicing Technique 1: Match Capture Slice StripWidth to match the pixel dimensions of AtlasWidth/NumHorizSteps & s.t. stripwidth/stripheight fovs match hAngIncr & vAngIncr
        //                     Legacy technique but allows setting the strip width to match atlas slice width
        //                     Pretty wasteful and will break if CaptureHFov & hangIncr/vAngIncr diverge greatly b/c resultant texture will exceed GPU bounds
        //                     StripHeight is computed based on solving CpxV = CpxH * SpxV / SpxH
        //                                                               CpxV = CV   * SpxV / SV
        //                                                               captureVfov = 2 * atan( tan(captureHfov / 2) * (SpxV / SpxH) )
        sliceHFov = hAngIncrement;
        sliceVFov = vAngIncrement;

        //TODO: ikrimae: Also do a quick test to see if there are issues with setting fov to something really small ( < 1 degree)
        //               And it does. Current noted issues: screen space effects like SSAO, AA, SSR are all off
        //                                                  local eyeadaptation also causes problems. Should probably turn off all PostProcess effects
        //                                                  small fovs cause floating point errors in the sampling function (probably a bug b/c no thought put towards that)
        captureHFov = FStereoPanoramaManager::CaptureHorizontalFOV->GetFloat();

        ensure(captureHFov >= hAngIncrement);

        //TODO: ikrimae: In hindsight, there's no reason that strip size should be this at all. Just select a square FOV larger than hAngIncr & vAngIncr
        //               and then sample the resulting plane accordingly. Remember when updating to this to recheck the math in resample function. Might
        //               have made assumptions about capture slice dimensions matching the sample strips
        StripWidth = SphericalAtlasWidth / NumberOfHorizontalSteps;
        //The scenecapture cube won't allow horizontal & vertical fov to not match the aspect ratio so we have to compute the right dimensions here for square pixels
        StripHeight = StripWidth * FMath::Tan(FMath::DegreesToRadians(vAngIncrement / 2.0f)) / FMath::Tan(FMath::DegreesToRadians(hAngIncrement / 2.0f));

        const FVector2D slicePlaneDim = FVector2D(
            2.0f * FMath::Tan(FMath::DegreesToRadians(hAngIncrement) / 2.0f),
            2.0f * FMath::Tan(FMath::DegreesToRadians(vAngIncrement) / 2.0f));

        const float capturePlaneWidth = 2.0f * FMath::Tan(FMath::DegreesToRadians(captureHFov) / 2.0f);

        //TODO: ikrimae: This is just to let the rest of the existing code work. Sampling rate of the slice can be whatever.
        //      Ex: To match the highest sampling frequency of the spherical atlas, it should match the area of differential patch
        //      at ray direction of pixel(0,1) in the atlas

        //Need stripwidth/slicePlaneDim.X = capturewidth / capturePlaneDim.X
        CaptureWidth = capturePlaneWidth * StripWidth / slicePlaneDim.X;
        CaptureHeight = CaptureWidth * StripHeight / StripWidth;

        captureVFov = FMath::RadiansToDegrees(2 * FMath::Atan(FMath::Tan(FMath::DegreesToRadians(captureHFov / 2.0f)) * CaptureHeight / CaptureWidth));

        //float dbgCapturePlaneDimY = 2.0f * FMath::Tan(FMath::DegreesToRadians(captureVFov) / 2.0f);
        //float dbgCaptureHeight = dbgCapturePlaneDimY * StripHeight / slicePlaneDim.Y;
    }
    else
    {
        //Slicing Technique 2: Each slice is a determined square FOV at a configured preset resolution.
        //                     Strip Width/Strip Height is determined based on hAngIncrement & vAngIncrement
        //                     Just make sure pixels/captureHFov >= pixels/hAngIncr && pixels/vAngIncr

        captureVFov = captureHFov = FStereoPanoramaManager::CaptureHorizontalFOV->GetFloat();
        sliceVFov   = sliceHFov   = captureHFov;

        ensure(captureHFov >= FMath::Max(hAngIncrement, vAngIncrement));
        
        //TODO: ikrimae: Re-do for floating point accuracy
        const FVector2D slicePlaneDim = FVector2D(
            2.0f * FMath::Tan(FMath::DegreesToRadians(hAngIncrement) / 2.0f),
            2.0f * FMath::Tan(FMath::DegreesToRadians(vAngIncrement) / 2.0f));

        const FVector2D capturePlaneDim = FVector2D(
            2.0f * FMath::Tan(FMath::DegreesToRadians(captureHFov) / 2.0f),
            2.0f * FMath::Tan(FMath::DegreesToRadians(captureVFov) / 2.0f));

        CaptureHeight = CaptureWidth = FStereoPanoramaManager::CaptureSlicePixelWidth->GetInt();

        StripWidth  = CaptureWidth  * slicePlaneDim.X / capturePlaneDim.X;
        StripHeight = CaptureHeight * slicePlaneDim.Y / capturePlaneDim.Y;

        //TODO: ikrimae: Come back and check for the actual right sampling rate
        check(StripWidth  >=  (SphericalAtlasWidth / NumberOfHorizontalSteps) && 
              StripHeight >= (SphericalAtlasHeight / NumberOfVerticalSteps));
        
        //Ensure Width/Height is always even
        StripWidth  += StripWidth & 1;
        StripHeight += StripHeight & 1;

    }

    UnprojectedAtlasWidth  = NumberOfHorizontalSteps * StripWidth;
    UnprojectedAtlasHeight = NumberOfVerticalSteps   * StripHeight;

    //NOTE: ikrimae: Ensure that the main gameview is > CaptureWidth x CaptureHeight. Bug in Unreal Engine that won't re-alloc scene render targets to the correct size
    //               when the scenecapture component > current window render target. https://answers.unrealengine.com/questions/80531/scene-capture-2d-max-resolution.html
    //TODO: ikrimae: Ensure that r.SceneRenderTargetResizeMethod=2
    FSystemResolution::RequestResolutionChange(CaptureWidth, CaptureHeight, EWindowMode::Windowed);

	// Creating CaptureSceneComponent to use it as parent scene component.
	// This scene component will hold same world location from camera.
	// Camera rotation will be used following UseCameraRotation settings.
	// Then, angular step turn will be applied to capture components locally to simplify calculation step that finding proper rotation.
	CaptureSceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("CaptureSceneComponent"));
	CaptureSceneComponent->AddToRoot();

	for( int CaptureIndex = 0; CaptureIndex < FStereoPanoramaManager::ConcurrentCaptures->GetInt(); CaptureIndex++ )
	{
		// initialize left eye
		FString LeftCounter = FString::Printf(TEXT("LeftEyeCaptureComponent_%04d"), CaptureIndex);
		USceneCaptureComponent2D* LeftEyeCaptureComponent = CreateDefaultSubobject<USceneCaptureComponent2D>(*LeftCounter);
		LeftEyeCaptureComponent->bTickInEditor = false;
		LeftEyeCaptureComponent->SetComponentTickEnabled(false);
		LeftEyeCaptureComponent->AttachToComponent(CaptureSceneComponent, FAttachmentTransformRules::KeepRelativeTransform);
		InitCaptureComponent( LeftEyeCaptureComponent, captureHFov, captureVFov);
		LeftEyeCaptureComponents.Add( LeftEyeCaptureComponent );

		// initialize right eye
		FString RightCounter = FString::Printf(TEXT("RightEyeCaptureComponent_%04d"), CaptureIndex);
		USceneCaptureComponent2D* RightEyeCaptureComponent = CreateDefaultSubobject<USceneCaptureComponent2D>(*RightCounter);
		RightEyeCaptureComponent->bTickInEditor = false;
		RightEyeCaptureComponent->SetComponentTickEnabled(false);
		RightEyeCaptureComponent->AttachToComponent(CaptureSceneComponent, FAttachmentTransformRules::KeepRelativeTransform);
		InitCaptureComponent(RightEyeCaptureComponent, captureHFov, captureVFov);
		RightEyeCaptureComponents.Add(RightEyeCaptureComponent);
	}

	CurrentStep = 0;
	TotalSteps = 0;
	FrameDescriptors = TEXT( "FrameNumber, GameClock, TimeTaken(s)" LINE_TERMINATOR );
	CaptureStep = ECaptureStep::Reset;

	// populate RenderPasses based on user options
	CurrentRenderPassIndex = 0;
	if (bOutputFinalColor)
	{
		RenderPasses.Add(ERenderPass::FinalColor);
	}

	if (bOutputSceneDepth)
	{
		RenderPasses.Add(ERenderPass::SceneDepth);
	}

	if (bOutputWorldNormal)
	{
		RenderPasses.Add(ERenderPass::WorldNormal);
	}

	if (bOutputRoughness)
	{
		RenderPasses.Add(ERenderPass::Roughness);
	}

	if (bOutputMetallic)
	{
		RenderPasses.Add(ERenderPass::Metallic);
	}

	if (bOutputBaseColor)
	{
		RenderPasses.Add(ERenderPass::BaseColor);
	}

	if (bOutputAmbientOcclusion)
	{
		RenderPasses.Add(ERenderPass::AO);
	}

	// default behavior is to render finalcolor if no passes are specified
	if (RenderPasses.Num() == 0)
	{
		RenderPasses.Add(ERenderPass::FinalColor);
	}

}

// Output current render pass name as string
FString USceneCapturer::GetCurrentRenderPassName()
{
	FString RenderPassString;
	switch (RenderPasses[CurrentRenderPassIndex])
	{
		case ERenderPass::FinalColor: RenderPassString = "FinalColor"; break;
		case ERenderPass::WorldNormal: RenderPassString = "WorldNormal"; break;
		case ERenderPass::AO: RenderPassString = "AO"; break;
		case ERenderPass::BaseColor: RenderPassString = "BaseColor"; break;
		case ERenderPass::Metallic: RenderPassString = "Metallic"; break;
		case ERenderPass::Roughness: RenderPassString = "Roughness"; break;
		case ERenderPass::SceneDepth: RenderPassString = "SceneDepth"; break;
		default: RenderPassString = ""; break;
	}
	return RenderPassString;
}

UMaterial* USceneCapturer::GetCurrentBlendableMaterial()
{
	UMaterial* CurrentBlendable;
	switch (RenderPasses[CurrentRenderPassIndex])
	{
		case ERenderPass::WorldNormal: CurrentBlendable = MaterialBlendableWorldNormal; break;
		case ERenderPass::AO: CurrentBlendable = MaterialBlendableAO; break;
		case ERenderPass::BaseColor: CurrentBlendable = MaterialBlendableBaseColor; break;
		case ERenderPass::Metallic: CurrentBlendable = MaterialBlendableMetallic; break;
		case ERenderPass::Roughness: CurrentBlendable = MaterialBlendableRoughness; break;
		default: CurrentBlendable = NULL; break;
	}
	return CurrentBlendable;
}

UWorld* USceneCapturer::GetTickableGameObjectWorld() const 
{
	// Check SceneCapturer have CaptureComponents and parent scene component is not marked as pending kill.
	if (LeftEyeCaptureComponents.Num() > 0 && IsValid(CaptureSceneComponent))
	{
		return CaptureSceneComponent->GetWorld();
	}
	return nullptr;
}

void USceneCapturer::Reset()
{
	// apply old states on PP volumes
	EnablePostProcessVolumes();
	
	for( int CaptureIndex = 0; CaptureIndex < FStereoPanoramaManager::ConcurrentCaptures->GetInt(); CaptureIndex++ )
	{
		USceneCaptureComponent2D* LeftEyeCaptureComponent = LeftEyeCaptureComponents[CaptureIndex];
		USceneCaptureComponent2D* RightEyeCaptureComponent = RightEyeCaptureComponents[CaptureIndex];

		LeftEyeCaptureComponent->SetVisibility( false );
		LeftEyeCaptureComponent->SetHiddenInGame( true );
		
		// Unreal Engine cannot serialize an array of subobject pointers, so work around the GC problems
		LeftEyeCaptureComponent->RemoveFromRoot();

		RightEyeCaptureComponent->SetVisibility( false );
		RightEyeCaptureComponent->SetHiddenInGame( true );
		
		// Unreal Engine cannot serialize an array of subobject pointers, so work around the GC problems
		RightEyeCaptureComponent->RemoveFromRoot();
	}

	UnprojectedLeftEyeAtlas.Empty();
	UnprojectedRightEyeAtlas.Empty();
	CaptureSceneComponent->RemoveFromRoot();
}

// Make sure we remove all blendables
void USceneCapturer::RemoveAllBlendables(USceneCaptureComponent2D* CaptureComponent)
{
	CaptureComponent->PostProcessSettings.RemoveBlendable(MaterialBlendableWorldNormal);
	CaptureComponent->PostProcessSettings.RemoveBlendable(MaterialBlendableAO);
	CaptureComponent->PostProcessSettings.RemoveBlendable(MaterialBlendableBaseColor);
	CaptureComponent->PostProcessSettings.RemoveBlendable(MaterialBlendableMetallic);
	CaptureComponent->PostProcessSettings.RemoveBlendable(MaterialBlendableRoughness);
}

// Disable screen space post processes we cannot use while capturing
void USceneCapturer::DisableUnsupportedPostProcesses(USceneCaptureComponent2D* CaptureComponent)
{
	CaptureComponent->PostProcessSettings.bOverride_FilmGrainIntensity = true;
	CaptureComponent->PostProcessSettings.FilmGrainIntensity = 0.0f;
	CaptureComponent->PostProcessSettings.bOverride_MotionBlurAmount = true;
	CaptureComponent->PostProcessSettings.MotionBlurAmount = 0.0f;
	CaptureComponent->PostProcessSettings.bOverride_ScreenSpaceReflectionIntensity = true;
	CaptureComponent->PostProcessSettings.ScreenSpaceReflectionIntensity = 0.0f;
	CaptureComponent->PostProcessSettings.bOverride_VignetteIntensity = true;
	CaptureComponent->PostProcessSettings.VignetteIntensity = 0.0f;
	CaptureComponent->PostProcessSettings.bOverride_LensFlareIntensity = true;
	CaptureComponent->PostProcessSettings.LensFlareIntensity = 0.0f;
}

// Cache all PP volumes in scene and save current "enable" state
void USceneCapturer::CacheAllPostProcessVolumes()
{
	TArray<AActor*> AllActors;
	UGameplayStatics::GetAllActorsOfClass(GetWorld(), APostProcessVolume::StaticClass(), AllActors);
	for (AActor* pp : AllActors)
	{
		APostProcessVolume * PPVolumeObject = CastChecked<APostProcessVolume>(pp);
		FPostProcessVolumeData PPVolumeData;
		PPVolumeData.Object = PPVolumeObject;
		PPVolumeData.WasEnabled = PPVolumeObject->bEnabled;
		PPVolumeArray.Add(PPVolumeData);
	}
}

// Apply old state on PP volumes
void USceneCapturer::EnablePostProcessVolumes()
{
	for (FPostProcessVolumeData &PPVolume : PPVolumeArray)
	{
		PPVolume.Object->bEnabled = PPVolume.WasEnabled;
	}
}

// Disable all PP volumes in scene to make sure g-buffer passes work
void USceneCapturer::DisableAllPostProcessVolumes()
{
	for (FPostProcessVolumeData &PPVolume : PPVolumeArray)
	{
		PPVolume.Object->bEnabled = false;
	}
}

// setup capture component based on current render pass
void USceneCapturer::SetCaptureComponentRequirements(int32 CaptureIndex)
{
	// remove all blendables
	RemoveAllBlendables(LeftEyeCaptureComponents[CaptureIndex]);
	RemoveAllBlendables(RightEyeCaptureComponents[CaptureIndex]);

	// FINAL COLOR
	if (RenderPasses[CurrentRenderPassIndex] == ERenderPass::FinalColor)
	{
		LeftEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = false;			// true will create bandings, false wont pick up blendables
		LeftEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;
		RightEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = false;		// true will create bandings, false wont pick up blendables
		RightEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;

		// Enable bCaptureEveryFrame ONLY when capture is in a PPVolume with blendables, to avoid bandings
		FVector CameraPosition = CapturePlayerController->PlayerCameraManager->GetCameraLocation();
		for (FPostProcessVolumeData &PPVolume : PPVolumeArray)
		{
			FBoxSphereBounds bounds = PPVolume.Object->GetBounds();
			FBox BB = bounds.GetBox();
			if (BB.IsInsideOrOn(CameraPosition))
			{
				//LeftEyeCaptureComponents[CaptureIndex]->PostProcessSettings.WeightedBlendables = ppvolume->Settings.WeightedBlendables;
				if (PPVolume.Object->bEnabled && PPVolume.Object->Settings.WeightedBlendables.Array.Num() > 0)
				{
					LeftEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = true;
					RightEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = true;
				}
			}

		}
	}

	// SCENE DEPTH
	if (RenderPasses[CurrentRenderPassIndex] == ERenderPass::SceneDepth)
	{
		LeftEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = false;
		LeftEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_DeviceDepth;
		RightEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = false;
		RightEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_DeviceDepth;
	}

	// All other passes
	if (RenderPasses[CurrentRenderPassIndex] < ERenderPass::FinalColor)
	{
		LeftEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = true;										// "true" for blendable to work
		LeftEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;			// "SCS_FinalColorLDR" for blendable to work
		LeftEyeCaptureComponents[CaptureIndex]->PostProcessSettings.AddBlendable(GetCurrentBlendableMaterial(), 1.0f);		// add postprocess material
		RightEyeCaptureComponents[CaptureIndex]->bCaptureEveryFrame = true;										// "true" for blendable to work
		RightEyeCaptureComponents[CaptureIndex]->CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;		// "SCS_FinalColorLDR" for blendable to work
		RightEyeCaptureComponents[CaptureIndex]->PostProcessSettings.AddBlendable(GetCurrentBlendableMaterial(), 1.0f);	// add postprocess material
	}
	
	DisableUnsupportedPostProcesses(LeftEyeCaptureComponents[CaptureIndex]);
	DisableUnsupportedPostProcesses(RightEyeCaptureComponents[CaptureIndex]);
}


void USceneCapturer::SetPositionAndRotation( int32 CurrentHorizontalStep, int32 CurrentVerticalStep, int32 CaptureIndex )
{
	// Using FRotator Rotation to hold local rotation.
	FRotator Rotation = FRotator(90.0f - 1.0f * CurrentVerticalStep * vAngIncrement, 180.0f + CurrentHorizontalStep * hAngIncrement, 0);

    Rotation = Rotation.Clamp();

	FVector Offset( 0.0f, eyeSeparation / 2.0f, 0.0f );
    if (dbgDisableOffsetRotation)
    {
        //For rendering near field objects, we don't rotate the capture components around the stereo pivot, but instead
        //around each capture component
        const auto rotAngleOffset = FRotator::ClampAxis(Rotation.Yaw - StartRotation.Yaw);
        float eyeSeparationDampeningFactor = 1.0f;
        if (rotAngleOffset <= 90.0f)
        {
            eyeSeparationDampeningFactor = FMath::Lerp(1.0f, 0.0f, rotAngleOffset / 90.0f);
        }
        else if (rotAngleOffset <= 270.0f)
        {
            eyeSeparationDampeningFactor = 0.0f;
        }
        else
        {
            eyeSeparationDampeningFactor = FMath::Lerp(0.0f, 1.0f, (rotAngleOffset - 270.0f) / 90.0f);
        }

        Offset = StartRotation.RotateVector(Offset * eyeSeparationDampeningFactor);
    }
    else
    {
        Offset = Rotation.RotateVector(Offset);
    }

	// Applying local offsets.
	// Rotation will be used as local rotation to make it regardless of World Rotation.
	// Local location will be used to set eye offset.
	LeftEyeCaptureComponents[CaptureIndex]->SetRelativeLocationAndRotation( -1.0f * Offset, Rotation );
    LeftEyeCaptureComponents[CaptureIndex]->CaptureSceneDeferred();

	// Do not process right eye in mono
	if (!bMonoscopicMode)
	{
		RightEyeCaptureComponents[CaptureIndex]->SetRelativeLocationAndRotation(1.0f * Offset, Rotation);
		RightEyeCaptureComponents[CaptureIndex]->CaptureSceneDeferred();
	}
}

void USceneCapturer::ValidateParameters()
{
	ErrorFound = false;

	FFormatNamedArguments Args;
	Args.Add(TEXT("CaptureWidth"), CaptureWidth);
	Args.Add(TEXT("CaptureHeight"), CaptureHeight);
	Args.Add(TEXT("SphericalAtlasWidth"), SphericalAtlasWidth);
	Args.Add(TEXT("SphericalAtlasHeight"), SphericalAtlasHeight);
	Args.Add(TEXT("UnprojectedAtlasWidth"), UnprojectedAtlasWidth);
	Args.Add(TEXT("UnprojectedAtlasHeight"), UnprojectedAtlasHeight);
	Args.Add(TEXT("StripWidth"), StripWidth);
	Args.Add(TEXT("StripHeight"), StripHeight);
	Args.Add(TEXT("NumberOfHorizontalSteps"), NumberOfHorizontalSteps);
	Args.Add(TEXT("NumberOfVerticalSteps"), NumberOfVerticalSteps);
	Args.Add(TEXT("vAngIncrement"), vAngIncrement);
	Args.Add(TEXT("hAngIncrement"), hAngIncrement);


	// check if we output renders in an existing drive
	int32 Index = OutputDir.Find(TEXT(":"), ESearchCase::CaseSensitive);
	FString Drive = OutputDir.Left(Index + 1);
	if (!Drive.IsEmpty() && FPaths::IsDrive(Drive))
	{
		if (!FPaths::DirectoryExists(Drive + FGenericPlatformMisc::GetDefaultPathSeparator()))
		{
			ErrorFound = true;
			FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Error, LOCTEXT("ValidationError_MissingOutputDirectory", "The output directory's drive doesn't exists. Plese set SP.OutputDir with a valid path. Skipping renders..."));
		}
	}

	// check if we have found all blendables
	if (MaterialBlendableAO == NULL || MaterialBlendableBaseColor == NULL || MaterialBlendableMetallic == NULL || MaterialBlendableRoughness == NULL || MaterialBlendableWorldNormal == NULL)
	{
		ErrorFound = true;
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Error, LOCTEXT("ValidationError_MissingBlendableMaterials", "Missing blendable materials. Skipping renders..."));
	}

	// Angular increment needs to be a factor of 360 to avoid seams i.e. 360 / angular increment needs to be a whole number
	if ((int32)(NumberOfHorizontalSteps * hAngIncrement) != 360)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidHorizonalAngularStep", "Horizontal angular step {hAngIncrement} is not a factor of 360! This will lead to a seam between the start and end points"), Args));
	}

	if ((int32)((NumberOfVerticalSteps - 1) * vAngIncrement) != 180)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidVerticalAngularStep", "Vertical angular step {vAngIncrement} is not a factor of 180! This will lead to a seam between the start and end points"), Args));
	}

	TotalSteps = NumberOfHorizontalSteps * NumberOfVerticalSteps;
	if ((SphericalAtlasWidth & 1) != 0)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidAtlasWidth", "The Atlas Width {SphericalAtlasWidth} must be even! Otherwise the Atlas height will not divide evenly."), Args));
	}

	// The strip width needs to be an even number and a factor of the number of steps
	if ((StripWidth & 1) != 0)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidStripWidth", "Strip width {StripWidth} needs to be even to avoid bad offsets"), Args));
	}

	if ((StripHeight & 1) != 0)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidStripHeight", "Strip height {StripHeight} needs to be even to avoid bad offsets"), Args));
	}

	// Validate capturewidth & captureheight. Need to be even
	if (CaptureWidth % 2 != 0)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidCaptureWidth", "The {CaptureWidth} needs to be an even number"), Args));
	}

	if (CaptureHeight % 2 != 0)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidCaptureHeight", "The {CaptureHeight} needs to be an even number"), Args));

	}

	// Unnecessary warning
	//if (StripWidth * NumberOfHorizontalSteps != SphericalAtlasWidth)
	//{
		//FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidNumHorizonalSteps", "The number of horizontal steps {NumberOfHorizontalSteps} needs to be a factor of the atlas width {SphericalAtlasWidth}"), Args));
	//}

	// Unnecessary warning
	//if (StripHeight * (NumberOfVerticalSteps - 1) != SphericalAtlasHeight)
	//{
		//FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, FText::Format(LOCTEXT("ValidationWarning_InvalidNumVerticalSteps", "The number of vertical steps {NumberOfVerticalSteps} needs to be a factor of the atlas height {SphericalAtlasHeight}"), Args));
	//}

	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("ValidationInfo_PanoramicScreenshotParams", "Panoramic screenshot parameters"));

	if (bMonoscopicMode)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("ValidationInfo_MonoscopicMode", " ... In Monoscopic mode"));
	}
	else
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("ValidationInfo_StereoscopicMode", " ... In Stereoscopic mode"));
	}

	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_CaptureSize", " ... capture size: {CaptureWidth} x {CaptureHeight}"), Args));
	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_SphericalAtlasSize", " ... spherical atlas size: {SphericalAtlasWidth} x {SphericalAtlasHeight}"), Args));
	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_IntermediateAtlasSize", " ... intermediate atlas size: {UnprojectedAtlasWidth} x {UnprojectedAtlasHeight}"), Args));
	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_StripSize", " ... strip size: {StripWidth} x {StripHeight}"), Args));
	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_NumHorizonalSteps", " ... horizontal steps: {NumberOfHorizontalSteps} at {hAngIncrement} degrees"), Args));
	FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("ValidationInfo_NumVerticalSteps", " ... vertical steps: {NumberOfVerticalSteps} at {vAngIncrement} degrees"), Args));
}

void USceneCapturer::SetInitialState( int32 InStartFrame, int32 InEndFrame, FStereoCaptureDoneDelegate& InStereoCaptureDoneDelegate )
{
	if( bIsTicking )
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, LOCTEXT("InitialStateWarning_AlreadyCapturing", "Already capturing a scene; concurrent captures are not allowed"));
		return;
	}

	CapturePlayerController = UGameplayStatics::GetPlayerController(GetWorld(), 0 ); //GWorld
	CaptureGameMode = UGameplayStatics::GetGameMode(GetWorld()); //GWorld

	if( CaptureGameMode == NULL || CapturePlayerController == NULL )
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, LOCTEXT("InitialStateWarning_MissingGameModeOrPC", "Missing GameMode or PlayerController"));
		return;
	}

	// Calculate the steps and validate they will produce good results
	ValidateParameters();

	// Setup starting criteria
    StartFrame				= InStartFrame;
    EndFrame				= InEndFrame;
	CurrentFrameCount		= 0;
    CurrentStep				= 0;
    CaptureStep				= ECaptureStep::Unpause;
	CurrentRenderPassIndex	= 0;

	Timestamp = FString::Printf( TEXT( "%s" ), *FDateTime::Now().ToString() );
	
	//SetStartPosition();

	// Create storage for atlas textures
    check( UnprojectedAtlasWidth * UnprojectedAtlasHeight <= MAX_int32 );
	UnprojectedLeftEyeAtlas.AddUninitialized(  UnprojectedAtlasWidth * UnprojectedAtlasHeight );
    UnprojectedRightEyeAtlas.AddUninitialized( UnprojectedAtlasWidth * UnprojectedAtlasHeight );

	StartTime        = FDateTime::UtcNow();
	OverallStartTime = StartTime;
	bIsTicking       = true;

    StereoCaptureDoneDelegate = InStereoCaptureDoneDelegate;

	// open log on error
	if (FMessageLog(StereoPanoramaLogName).NumMessages(EMessageSeverity::Error) > 0)
	{
		FMessageLog(StereoPanoramaLogName).Open();
	}

}

void USceneCapturer::CopyToUnprojAtlas( int32 CurrentHorizontalStep, int32 CurrentVerticalStep, TArray<FLinearColor>& Atlas, TArray<FLinearColor>& SurfaceData )
{
	int32 XOffset = StripWidth * CurrentHorizontalStep;
    int32 YOffset = StripHeight * CurrentVerticalStep;

	int32 StripSize = StripWidth * sizeof(FLinearColor);
    for (int32 Y = 0; Y < StripHeight; Y++)
	{
        void* Destination = &Atlas[( ( Y + YOffset ) * UnprojectedAtlasWidth ) + XOffset];
		void* Source = &SurfaceData[StripWidth * Y];
		FMemory::Memcpy( Destination, Source, StripSize );
	}
}

TArray<FLinearColor> USceneCapturer::SaveAtlas(FString Folder, const TArray<FLinearColor>& SurfaceData)
{
	SCOPE_CYCLE_COUNTER(STAT_SPSavePNG);

	TArray<FLinearColor> SphericalAtlas;
	SphericalAtlas.AddZeroed(SphericalAtlasWidth * SphericalAtlasHeight);
	
	const FVector2D slicePlaneDim = FVector2D(
		2.0f * FMath::Tan(FMath::DegreesToRadians(sliceHFov) / 2.0f),
		2.0f * FMath::Tan(FMath::DegreesToRadians(sliceVFov) / 2.0f));

	//For each direction,
	//    Find corresponding slice
	//    Calculate intersection of slice plane
	//    Calculate intersection UVs by projecting onto plane tangents
	//    Supersample that UV coordinate from the unprojected atlas
	{
		SCOPE_CYCLE_COUNTER(STAT_SPSampleSpherical);
		// Dump out how long the process took
		const FDateTime SamplingStartTime = FDateTime::UtcNow();
		for (int32 y = 0; y < SphericalAtlasHeight; y++)
		{
			for (int32 x = 0; x < SphericalAtlasWidth; x++)
			{
				FLinearColor samplePixelAccum = FLinearColor(0, 0, 0, 0);

				//TODO: ikrimae: Seems that bilinear filtering sans supersampling is good enough. Supersampling sans bilerp seems best.
				//               After more tests, come back to optimize by folding supersampling in and remove this outer sampling loop.
				const SamplingPattern& ssPattern = g_ssPatterns[SSMethod];

				for (int32 SampleCount = 0; SampleCount < ssPattern.numSamples; SampleCount++)
				{
					const float sampleU = ((float)x + ssPattern.ssOffsets[SampleCount].X) / SphericalAtlasWidth;
					const float sampleV = ((float)y + ssPattern.ssOffsets[SampleCount].Y) / SphericalAtlasHeight;

					const float sampleTheta = sampleU * 360.0f;
					const float samplePhi = sampleV * 180.0f;

					const FVector sampleDir = FVector(
						FMath::Sin(FMath::DegreesToRadians(samplePhi)) * FMath::Cos(FMath::DegreesToRadians(sampleTheta)),
						FMath::Sin(FMath::DegreesToRadians(samplePhi)) * FMath::Sin(FMath::DegreesToRadians(sampleTheta)),
						FMath::Cos(FMath::DegreesToRadians(samplePhi)));

					const int32 sliceXIndex = FMath::TruncToInt(FRotator::ClampAxis(sampleTheta + hAngIncrement / 2.0f) / hAngIncrement);
					int32 sliceYIndex = 0;

					//Slice Selection = slice with max{sampleDir dot  sliceNormal }
					{
						float largestCosAngle = 0;
						for (int VerticalStep = 0; VerticalStep < NumberOfVerticalSteps; VerticalStep++)
						{
							const FVector2D sliceCenterThetaPhi = FVector2D(
								hAngIncrement * sliceXIndex,
								vAngIncrement * VerticalStep);

							//TODO: ikrimae: There has got to be a faster way. Rethink reparametrization later
							const FVector sliceDir = FVector(
								FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
								FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
								FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)));

							const float cosAngle = sampleDir | sliceDir;

							if (cosAngle > largestCosAngle)
							{
								largestCosAngle = cosAngle;
								sliceYIndex = VerticalStep;
							}
						}
					}


					const FVector2D sliceCenterThetaPhi = FVector2D(
						hAngIncrement * sliceXIndex,
						vAngIncrement * sliceYIndex);

					//TODO: ikrimae: Reparameterize with an inverse mapping (e.g. project from slice pixels onto final u,v coordinates.
					//               Should make code simpler and faster b/c reduces to handful of sin/cos calcs per slice. 
					//               Supersampling will be more difficult though.

					const FVector sliceDir = FVector(
						FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
						FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
						FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)));

					const FPlane slicePlane = FPlane(sliceDir, -sliceDir);

					//Tangents from partial derivatives of sphere equation
					const FVector slicePlanePhiTangent = FVector(
						FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
						FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
						-FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y))).GetSafeNormal();

					//Should be reconstructed to get around discontinuity of theta tangent at nodal points
					const FVector slicePlaneThetaTangent = (sliceDir ^ slicePlanePhiTangent).GetSafeNormal();
					//const FVector slicePlaneThetaTangent = FVector(
					//    -FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
					//    FMath::Sin(FMath::DegreesToRadians(sliceCenterThetaPhi.Y)) * FMath::Cos(FMath::DegreesToRadians(sliceCenterThetaPhi.X)),
					//    0).SafeNormal();

					check(!slicePlaneThetaTangent.IsZero() && !slicePlanePhiTangent.IsZero());

					const double t = (double)-slicePlane.W / (sampleDir | sliceDir);
					const FVector sliceIntersection = FVector(t * sampleDir.X, t * sampleDir.Y, t * sampleDir.Z);

					//Calculate scalar projection of sliceIntersection onto tangent vectors. a dot b / |b| = a dot b when tangent vectors are normalized
					//Then reparameterize to U,V of the sliceplane based on slice plane dimensions
					const float sliceU = (sliceIntersection | slicePlaneThetaTangent) / slicePlaneDim.X;
					const float sliceV = (sliceIntersection | slicePlanePhiTangent) / slicePlaneDim.Y;

					check(sliceU >= -(0.5f + KINDA_SMALL_NUMBER) &&
						sliceU <= (0.5f + KINDA_SMALL_NUMBER));

					check(sliceV >= -(0.5f + KINDA_SMALL_NUMBER) &&
						sliceV <= (0.5f + KINDA_SMALL_NUMBER));

					//TODO: ikrimae: Supersample/bilinear filter
					const int32 slicePixelX = FMath::TruncToInt(dbgMatchCaptureSliceFovToAtlasSliceFov ? sliceU * StripWidth : sliceU * CaptureWidth);
					const int32 slicePixelY = FMath::TruncToInt(dbgMatchCaptureSliceFovToAtlasSliceFov ? sliceV * StripHeight : sliceV * CaptureHeight);

					FLinearColor slicePixelSample;

					if (bEnableBilerp)
					{
						//TODO: ikrimae: Clean up later; too tired now
						const int32 sliceCenterPixelX = (sliceXIndex + 0.5f) * StripWidth;
						const int32 sliceCenterPixelY = (sliceYIndex + 0.5f) * StripHeight;

						const FIntPoint atlasSampleTL(sliceCenterPixelX + FMath::Clamp(slicePixelX, -StripWidth / 2, StripWidth / 2), sliceCenterPixelY + FMath::Clamp(slicePixelY, -StripHeight / 2, StripHeight / 2));
						const FIntPoint atlasSampleTR(sliceCenterPixelX + FMath::Clamp(slicePixelX + 1, -StripWidth / 2, StripWidth / 2), sliceCenterPixelY + FMath::Clamp(slicePixelY, -StripHeight / 2, StripHeight / 2));
						const FIntPoint atlasSampleBL(sliceCenterPixelX + FMath::Clamp(slicePixelX, -StripWidth / 2, StripWidth / 2), sliceCenterPixelY + FMath::Clamp(slicePixelY + 1, -StripHeight / 2, StripHeight / 2));
						const FIntPoint atlasSampleBR(sliceCenterPixelX + FMath::Clamp(slicePixelX + 1, -StripWidth / 2, StripWidth / 2), sliceCenterPixelY + FMath::Clamp(slicePixelY + 1, -StripHeight / 2, StripHeight / 2));

						const FLinearColor pixelColorTL = SurfaceData[atlasSampleTL.Y * UnprojectedAtlasWidth + atlasSampleTL.X];
						const FLinearColor pixelColorTR = SurfaceData[atlasSampleTR.Y * UnprojectedAtlasWidth + atlasSampleTR.X];
						const FLinearColor pixelColorBL = SurfaceData[atlasSampleBL.Y * UnprojectedAtlasWidth + atlasSampleBL.X];
						const FLinearColor pixelColorBR = SurfaceData[atlasSampleBR.Y * UnprojectedAtlasWidth + atlasSampleBR.X];

						const float fracX = FMath::Frac(dbgMatchCaptureSliceFovToAtlasSliceFov ? sliceU * StripWidth : sliceU * CaptureWidth);
						const float fracY = FMath::Frac(dbgMatchCaptureSliceFovToAtlasSliceFov ? sliceV * StripHeight : sliceV * CaptureHeight);

						//Reinterpret as linear (a.k.a dont apply srgb inversion)
						slicePixelSample = FMath::BiLerp(pixelColorTL, pixelColorTR, pixelColorBL, pixelColorBR, fracX, fracY);
					}
					else
					{
						const int32 sliceCenterPixelX = (sliceXIndex + 0.5f) * StripWidth;
						const int32 sliceCenterPixelY = (sliceYIndex + 0.5f) * StripHeight;

						const int32 atlasSampleX = sliceCenterPixelX + slicePixelX;
						const int32 atlasSampleY = sliceCenterPixelY + slicePixelY;


						slicePixelSample = SurfaceData[atlasSampleY * UnprojectedAtlasWidth + atlasSampleX];
					}

					// accum
					samplePixelAccum += slicePixelSample;
				}

				SphericalAtlas[y * SphericalAtlasWidth + x] = samplePixelAccum / ssPattern.numSamples;

				// Force alpha value
				if (bForceAlpha)
				{
					SphericalAtlas[y * SphericalAtlasWidth + x].A = 1.0f;	// 255 in 8bit FColor
				}
			}
		}

		//Blit the first column into the last column to make the stereo image seamless at theta=360
		for (int32 y = 0; y < SphericalAtlasHeight; y++)
		{
			SphericalAtlas[y * SphericalAtlasWidth + (SphericalAtlasWidth - 1)] = SphericalAtlas[y * SphericalAtlasWidth + 0];
		}

		const FTimespan SamplingDuration = FDateTime::UtcNow() - SamplingStartTime;
	}

	// DEBUG ONLY
	if (FStereoPanoramaManager::GenerateDebugImages->GetInt() != 0)
	{
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);
		FString FrameStringUnprojected = FString::Printf(TEXT("%s_%05d_Unprojected.png"), *Folder, CurrentFrameCount);
		FString AtlasNameUnprojected = OutputDir / Timestamp / FrameStringUnprojected;
		ImageWrapper->SetRaw(SurfaceData.GetData(), SurfaceData.GetAllocatedSize(), UnprojectedAtlasWidth, UnprojectedAtlasHeight, ERGBFormat::BGRA, 32);
		const TArray64<uint8> PNGDataUnprojected = ImageWrapper->GetCompressed(100);
		FFileHelper::SaveArrayToFile(PNGDataUnprojected, *AtlasNameUnprojected);
		ImageWrapper.Reset();
	}
	
	return SphericalAtlas;
}


void USceneCapturer::CaptureComponent(int32 CurrentHorizontalStep, int32 CurrentVerticalStep, FString Folder, USceneCaptureComponent2D* CaptureComponent, TArray<FLinearColor>& Atlas)
{
	TArray<FLinearColor> SurfaceData;

	{
		SCOPE_CYCLE_COUNTER(STAT_SPReadStrip);

		FTextureRenderTargetResource* RenderTarget = CaptureComponent->TextureTarget->GameThread_GetRenderTargetResource();
		int32 CenterX = CaptureWidth / 2;
		int32 CenterY = CaptureHeight / 2;

		SurfaceData.AddUninitialized(StripWidth * StripHeight);

		// Read pixels
		FIntRect Area(CenterX - (StripWidth / 2), CenterY - (StripHeight / 2), CenterX + (StripWidth / 2), CenterY + (StripHeight / 2));
		FReadSurfaceDataFlags readSurfaceDataFlags = FReadSurfaceDataFlags();

		RenderTarget->ReadLinearColorPixelsPtr(SurfaceData.GetData(), readSurfaceDataFlags, Area);
	}

	// SceneDepth pass only
	if (RenderPasses[CurrentRenderPassIndex] == ERenderPass::SceneDepth)
	{
		// unpack 32bit scene depth from 4 channels RGBA
		for (FLinearColor& Color : SurfaceData)
		{
			Color.R = 1.0f - (Color.R + (Color.G / 255.0f) + (Color.B / 65025.0f) + (Color.A / 16581375.0f));	// unpack depth
			Color.R = FMath::Pow(Color.R, 0.4545f);																// linear to srgb
			Color.G = Color.R;
			Color.B = Color.R;
			Color.A = 1.0f;
		}
	}
	
	// Copy off strip to atlas texture
	CopyToUnprojAtlas(CurrentHorizontalStep, CurrentVerticalStep, Atlas, SurfaceData);

	// DEBUG ONLY
	if (FStereoPanoramaManager::GenerateDebugImages->GetInt() != 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_SPSavePNG);

		// Generate name
		FString TickString = FString::Printf(TEXT("_%05d_%04d_%04d"), CurrentFrameCount, CurrentHorizontalStep, CurrentVerticalStep);
		FString CaptureName = OutputDir / Timestamp / Folder / TickString + TEXT(".png");

		// Write out PNG
		if (FStereoPanoramaManager::GenerateDebugImages->GetInt() == 2)
		{
			//Read Whole Capture Buffer
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

			TArray<FLinearColor> SurfaceDataWhole;
			SurfaceDataWhole.AddUninitialized(CaptureWidth * CaptureHeight);
			
			// Read pixels
			FTextureRenderTargetResource* RenderTarget = CaptureComponent->TextureTarget->GameThread_GetRenderTargetResource();
			//RenderTarget->ReadPixelsPtr(SurfaceDataWhole, FReadSurfaceDataFlags());
			RenderTarget->ReadLinearColorPixelsPtr(SurfaceData.GetData(), FReadSurfaceDataFlags());

			// Force alpha value
			if (bForceAlpha)
			{
				for (FLinearColor& Color : SurfaceDataWhole)
				{
					Color.A = 1.0f;
				}
			}

			ImageWrapper->SetRaw(SurfaceDataWhole.GetData(), SurfaceDataWhole.GetAllocatedSize(), CaptureWidth, CaptureHeight, ERGBFormat::BGRA, 32);
			const TArray64<uint8> PNGData = ImageWrapper->GetCompressed(100);

			FFileHelper::SaveArrayToFile(PNGData, *CaptureName);
			ImageWrapper.Reset();
		}
		else
		{
			if (bForceAlpha)
			{
				for (FLinearColor& Color : SurfaceData)
				{
					Color.A = 1.0f;
				}
			}

			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);
			ImageWrapper->SetRaw(SurfaceData.GetData(), SurfaceData.GetAllocatedSize(), StripWidth, StripHeight, ERGBFormat::BGRA, 32);
			const TArray64<uint8> PNGData = ImageWrapper->GetCompressed(100);

			FFileHelper::SaveArrayToFile(PNGData, *CaptureName);
			ImageWrapper.Reset();
		}
	}
}


//TODO: ikrimae: Come back and actually work out the timings. Trickery b/c SceneCaptureCubes Tick at the end of the frame so we're effectively queuing up the next
//               step (pause, unpause, setposition) for the next frame. FlushRenderingCommands() added haphazardly to test but didn't want to remove them so close to delivery. 
//               Think through when we actually need to flush and document.
void USceneCapturer::Tick( float DeltaTime )
{
	// skip rendering if error found
	if (ErrorFound)
	{
		bIsTicking = false;
	}

	if( !bIsTicking )
	{
		return;
	}

    if ( CurrentFrameCount < StartFrame )
    {
        //Skip until we're at the frame we want to render
        CurrentFrameCount++;
        CaptureStep = ECaptureStep::Pause;
    }
	else if( CurrentStep < TotalSteps )
	{
        if (CaptureStep == ECaptureStep::Unpause)
        {
            FlushRenderingCommands();
            CaptureGameMode->ClearPause();

            CaptureStep = ECaptureStep::Pause;
            FlushRenderingCommands();
        }
        else if (CaptureStep == ECaptureStep::Pause)
        {
            FlushRenderingCommands();
			
			// To prevent following process when tick at the time of PIE ends and CaptureGameMode is no longer valid.
			if (!CaptureGameMode)
			{
				return;
			}

            CaptureGameMode->SetPause(CapturePlayerController);

            CaptureStep = ECaptureStep::SetStartPosition;
            FlushRenderingCommands();
        }
        else if (CaptureStep == ECaptureStep::SetStartPosition)
        {
            //SetStartPosition();
            ENQUEUE_RENDER_COMMAND(SceneCapturer_HeartbeatTickTickables)(
				[](FRHICommandListImmediate& RHICmdList)
				{
					TickRenderingTickables(RHICmdList);
				});

            FlushRenderingCommands();
            
            FRotator Rotation;
            CapturePlayerController->GetPlayerViewPoint(StartLocation, Rotation);
            // Gathering selected axis information from UseCameraRotation and saving it to FRotator Rotation.
			Rotation = FRotator(
				(UseCameraRotation & 1) ? Rotation.Pitch : 0.0f
				, (UseCameraRotation & 2) ? Rotation.Yaw : 0.0f
				, (UseCameraRotation & 4) ? Rotation.Roll : 0.0f
			);
            Rotation.Yaw = (bOverrideInitialYaw) ? ForcedInitialYaw : Rotation.Yaw;
            StartRotation = Rotation;

			// Set Designated Rotation and Location for CaptureSceneComponent, using it as parent scene component for capturecomponents.
			CaptureSceneComponent->SetWorldLocationAndRotation(StartLocation, Rotation);

			// set capture components settings before capturing and reading
			for (int32 CaptureIndex = 0; CaptureIndex < FStereoPanoramaManager::ConcurrentCaptures->GetInt(); CaptureIndex++)
			{
				SetCaptureComponentRequirements(CaptureIndex);
			}

			FString CurrentPassName = GetCurrentRenderPassName();
			FString Msg = FString("Processing pass: " + CurrentPassName);
			FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::FromString(Msg));

			if (RenderPasses[CurrentRenderPassIndex] != ERenderPass::FinalColor)
			{
				DisableAllPostProcessVolumes();
			}
			else
			{
				EnablePostProcessVolumes();
			}

            CaptureStep = ECaptureStep::SetPosition;
            FlushRenderingCommands();
        }
        else if (CaptureStep == ECaptureStep::SetPosition)
        {
            FlushRenderingCommands();
            for (int32 CaptureIndex = 0; CaptureIndex < FStereoPanoramaManager::ConcurrentCaptures->GetInt(); CaptureIndex++)
            {
                int32 CurrentHorizontalStep;
                int32 CurrentVerticalStep;
                if (GetComponentSteps(CurrentStep + CaptureIndex, CurrentHorizontalStep, CurrentVerticalStep))
                {
					SetPositionAndRotation(CurrentHorizontalStep, CurrentVerticalStep, CaptureIndex);
                }
            }

            CaptureStep = ECaptureStep::Read;
            FlushRenderingCommands();
        }
		else if (CaptureStep == ECaptureStep::Read)
		{
            FlushRenderingCommands();
            for (int32 CaptureIndex = 0; CaptureIndex < FStereoPanoramaManager::ConcurrentCaptures->GetInt(); CaptureIndex++)
            {
                int32 CurrentHorizontalStep;
                int32 CurrentVerticalStep;
				if (GetComponentSteps(CurrentStep, CurrentHorizontalStep, CurrentVerticalStep))
				{
					CaptureComponent(CurrentHorizontalStep, CurrentVerticalStep, TEXT("Left"), LeftEyeCaptureComponents[CaptureIndex], UnprojectedLeftEyeAtlas);
					if (!bMonoscopicMode)
					{
						CaptureComponent(CurrentHorizontalStep, CurrentVerticalStep, TEXT("Right"), RightEyeCaptureComponents[CaptureIndex], UnprojectedRightEyeAtlas);
					}
					CurrentStep++;
                }
            }

            CaptureStep = ECaptureStep::SetPosition;
            FlushRenderingCommands();
        }
        else
        {
            //ECaptureStep::Reset:
		}
	}
	else
	{
		TArray<FLinearColor> SphericalLeftEyeAtlas  = SaveAtlas(TEXT("Left"), UnprojectedLeftEyeAtlas);
		TArray<FLinearColor> SphericalRightEyeAtlas;

		if (CombineAtlasesOnOutput)
		{
			TArray<FLinearColor> CombinedAtlas;
			CombinedAtlas.Append(SphericalLeftEyeAtlas);
			int32 EyeCount = 1;

			// combine right eye in stereo
			if (!bMonoscopicMode)
			{
				SphericalRightEyeAtlas = SaveAtlas(TEXT("Right"), UnprojectedRightEyeAtlas);
				CombinedAtlas.Append(SphericalRightEyeAtlas);
				EyeCount++;
			}

			// Get current render pass name
			FString RenderPassString = GetCurrentRenderPassName();

			// save to disk
			if (OutputBitDepth == 8 && RenderPasses[CurrentRenderPassIndex] != ERenderPass::SceneDepth)
			{
				// switch to 8 bit/channel
				TArray<FColor> CombinedAtlas8bit;
				CombinedAtlas8bit.Empty( CombinedAtlas.Num() );
				for (FLinearColor& Color : CombinedAtlas)
				{
					CombinedAtlas8bit.Add( Color.QuantizeRound() );
				}

				// save as png 8bit/channel
				FString FrameString = FString::Printf(TEXT("Frame_%05d_%s.png"), CurrentFrameCount, *RenderPassString);
				FString AtlasName = OutputDir / Timestamp / RenderPassString / FrameString;
				FString Msg = FString("Writing file: " + AtlasName);
				FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::FromString(Msg));

				// write
				TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);
				ImageWrapper->SetRaw(CombinedAtlas8bit.GetData(), CombinedAtlas8bit.GetAllocatedSize(), SphericalAtlasWidth, SphericalAtlasHeight * EyeCount, ERGBFormat::BGRA, 8);
				const TArray64<uint8> ImageData = ImageWrapper->GetCompressed(100);
				FFileHelper::SaveArrayToFile(ImageData, *AtlasName);
				ImageWrapper.Reset();
				FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("Done", "Done!"));
			}			
			else
			{
				// stay in 32bit/channel
				FString FrameString = FString::Printf(TEXT("Frame_%05d_%s.exr"), CurrentFrameCount, *RenderPassString);
				FString AtlasName = OutputDir / Timestamp / RenderPassString / FrameString;
				FString Msg = FString("Writing file: " + AtlasName);
				FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::FromString(Msg));

				// write
				TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::EXR);
				ImageWrapper->SetRaw(CombinedAtlas.GetData(), CombinedAtlas.GetAllocatedSize(), SphericalAtlasWidth, SphericalAtlasHeight * EyeCount, ERGBFormat::RGBAF, 32);
				const TArray64<uint8> ImageData = ImageWrapper->GetCompressed();
				FFileHelper::SaveArrayToFile(ImageData, *AtlasName);
				ImageWrapper.Reset();
				FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("Done", "Done!"));
			}

		}
		
		// ----------------------------------------------------------------------------------------
		// Check if we need to do another render pass
		CurrentRenderPassIndex++;
		if (CurrentRenderPassIndex < RenderPasses.Num())
		{
			// NEXT RENDER PASS - same frame
			CurrentStep = 0;
			CaptureStep = ECaptureStep::SetStartPosition;
			UnprojectedLeftEyeAtlas.Empty();
			UnprojectedRightEyeAtlas.Empty();
			UnprojectedLeftEyeAtlas.AddUninitialized(UnprojectedAtlasWidth * UnprojectedAtlasHeight);
			UnprojectedRightEyeAtlas.AddUninitialized(UnprojectedAtlasWidth * UnprojectedAtlasHeight);
		}
		else
		{
			// ----------------------------------------------------------------------------------------
			// NEXT FRAME
			// Dump out how long the process took
			FDateTime EndTime = FDateTime::UtcNow();
			FTimespan Duration = EndTime - StartTime;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Duration"), Duration.GetTotalSeconds());
			Args.Add(TEXT("CurrentFrameCount"), CurrentFrameCount);
			FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("FrameDuration", "Duration: {Duration} seconds for frame {CurrentFrameCount}"), Args));
			
			StartTime = EndTime;

			//NOTE: ikrimae: Since we can't synchronously finish a stereocapture, we have to notify the caller with a function pointer
			//Not sure this is the cleanest way but good enough for now
			StereoCaptureDoneDelegate.ExecuteIfBound(SphericalLeftEyeAtlas, SphericalRightEyeAtlas);

			// Construct log of saved atlases in csv format
			FrameDescriptors += FString::Printf(TEXT("%d, %g, %g" LINE_TERMINATOR), CurrentFrameCount, FApp::GetCurrentTime() - FApp::GetLastTime(), Duration.GetTotalSeconds());

			CurrentFrameCount++;
			if (CurrentFrameCount <= EndFrame)
			{
				CurrentStep = 0;
				CaptureStep = ECaptureStep::Unpause;
				CurrentRenderPassIndex = 0;
			}
			else
			{
				// ----------------------------------------------------------------------------------------
				// EXIT 
				CaptureGameMode->ClearPause();

				FTimespan OverallDuration = FDateTime::UtcNow() - OverallStartTime;

				FrameDescriptors += FString::Printf(TEXT("Duration: %g minutes for frame range [%d,%d] "), OverallDuration.GetTotalMinutes(), StartFrame, EndFrame);
				
				FFormatNamedArguments ExitArgs;
				ExitArgs.Add(TEXT("Duration"), OverallDuration.GetTotalMinutes());
				ExitArgs.Add(TEXT("StartFrame"), StartFrame);
				ExitArgs.Add(TEXT("EndFrame"), EndFrame);
				FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, FText::Format(LOCTEXT("CompleteDuration", "Duration: {Duration} minutes for frame range [{StartFrame},{EndFrame}] "), ExitArgs));
				
				FString FrameDescriptorName = OutputDir / Timestamp / TEXT("Frames.txt");
				FFileHelper::SaveStringToFile(FrameDescriptors, *FrameDescriptorName, FFileHelper::EEncodingOptions::ForceUTF8);

				bIsTicking = false;
				FStereoPanoramaModule::Get()->Cleanup();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE