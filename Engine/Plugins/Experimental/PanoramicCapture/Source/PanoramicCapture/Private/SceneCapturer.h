// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/Material.h"
#include "Tickable.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Engine/PostProcessVolume.h"
#include "Logging/MessageLog.h"
#include "SceneCapturer.generated.h"

class IImageWrapperModule;

DECLARE_LOG_CATEGORY_EXTERN( LogStereoPanorama, Log, All );

DECLARE_STATS_GROUP( TEXT( "SP" ), STATGROUP_SP, STATCAT_Advanced );

DECLARE_CYCLE_STAT( TEXT( "SavePNG" ),         STAT_SPSavePNG,         STATGROUP_SP );
DECLARE_CYCLE_STAT( TEXT( "SampleSpherical" ), STAT_SPSampleSpherical, STATGROUP_SP );
DECLARE_CYCLE_STAT( TEXT( "ReadStrip" ),       STAT_SPReadStrip,       STATGROUP_SP );
DECLARE_CYCLE_STAT( TEXT( "FillAlpha" ),       STAT_SPFillAlpha,       STATGROUP_SP );

extern const FName StereoPanoramaLogName;

USTRUCT()
struct FPostProcessVolumeData
{
	GENERATED_BODY()
	
	UPROPERTY()
	TObjectPtr<APostProcessVolume> Object;

	bool WasEnabled;
};

enum class ECaptureStep : uint8
{
	Reset,
    SetStartPosition,
	SetPosition,
	Read,
	Pause,
	Unpause
};

enum class ERenderPass : uint8
{
	WorldNormal,
	AO,
	BaseColor,
	Metallic,
	Roughness,
	FinalColor,
	SceneDepth
};

DECLARE_DELEGATE_TwoParams(FStereoCaptureDoneDelegate, const TArray<FLinearColor>&, const TArray<FLinearColor>&);


UCLASS()
class USceneCapturer 
	: public UObject
	, public FTickableGameObject
{
    GENERATED_BODY()

public:

    USceneCapturer();

    //NOTE: ikrimae: Adding this ctor hack to fix the 4.8p2 problem with hot reload macros calling empty constructors
    //               https://answers.unrealengine.com/questions/228042/48p2-compile-fails-on-class-with-non-default-const.html
    USceneCapturer(FVTableHelper& Helper);
    
public:

	//~ FTickableGameObject interface

	virtual void Tick( float DeltaTime ) override;

	virtual ETickableTickType GetTickableTickType() const override
	{ 
		return ETickableTickType::Always; 
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return bIsTicking;
	}

	virtual UWorld* GetTickableGameObjectWorld() const override;

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT( USceneCapturer, STATGROUP_Tickables );
	}

public:

	void InitCaptureComponent( USceneCaptureComponent2D* CaptureComponent, float HFov, float VFov);

	void CaptureComponent(int32 CurrentHorizontalStep, int32 CurrentVerticalStep, FString Folder, USceneCaptureComponent2D* CaptureComponent, TArray<FLinearColor>& Atlas);

	void CopyToUnprojAtlas(int32 CurrentHorizontalStep, int32 CurrentVerticalStep, TArray<FLinearColor>& Atlas, TArray<FLinearColor>& SurfaceData);

	TArray<FLinearColor> SaveAtlas(FString Folder, const TArray<FLinearColor>& SurfaceData);

	void SetPositionAndRotation( int32 CurrentHorizontalStep, int32 CurrentVerticalStep, int32 CaptureIndex );

	void SetCaptureComponentRequirements( int32 CaptureIndex);

	void DisableAllPostProcessVolumes();

	void CacheAllPostProcessVolumes();

	void EnablePostProcessVolumes();

	void DisableUnsupportedPostProcesses(USceneCaptureComponent2D* CaptureComponent);

	void RemoveAllBlendables(USceneCaptureComponent2D* CaptureComponent);

	void ValidateParameters();

	void SetInitialState( int32 InStartFrame, int32 InEndFrame, FStereoCaptureDoneDelegate& InStereoCaptureDoneDelegate );

	FString GetCurrentRenderPassName();

	UMaterial* GetCurrentBlendableMaterial();

	void Reset();

public:

	IImageWrapperModule& ImageWrapperModule;

	bool bIsTicking;
	FDateTime OverallStartTime;
	FDateTime StartTime;

	FVector StartLocation;
	FRotator StartRotation;
	FString Timestamp;
    int32 StartFrame;
    int32 EndFrame;

	ECaptureStep CaptureStep;
	int32 CurrentFrameCount;

	int32 CaptureWidth;
	int32 CaptureHeight;

	int32 StripWidth;
    int32 StripHeight;

	// blendable materials for each pass
	UMaterial* MaterialBlendableWorldNormal;
	UMaterial* MaterialBlendableAO;
	UMaterial* MaterialBlendableBaseColor;
	UMaterial* MaterialBlendableMetallic;
	UMaterial* MaterialBlendableRoughness;

	// store which passes to do per frame
	TArray<ERenderPass> RenderPasses;
	int CurrentRenderPassIndex;

	// store post process volumes data
	UPROPERTY(Transient)
	TArray <FPostProcessVolumeData> PPVolumeArray;

	class APlayerController* CapturePlayerController;
	class AGameModeBase* CaptureGameMode;

	TArray<USceneCaptureComponent2D*> LeftEyeCaptureComponents;
	TArray<USceneCaptureComponent2D*> RightEyeCaptureComponents;
	
	// CaptureSceneComponent will be used as parent of capturecomponents to provide world location and rotation.
	UPROPERTY(Transient)
	TObjectPtr<USceneComponent> CaptureSceneComponent;

	bool GetComponentSteps( int32 Step, int32& CurrentHorizontalStep, int32& CurrentVerticalStep )
	{
		if( Step < TotalSteps )
		{
			CurrentHorizontalStep = Step / NumberOfVerticalSteps;
			CurrentVerticalStep = Step - ( CurrentHorizontalStep * NumberOfVerticalSteps );
			return true;
		}

		return false;
	}

private:

    const float hAngIncrement;
    const float vAngIncrement;
    float eyeSeparation;
    
    float sliceHFov;
    float sliceVFov;

    const int32 NumberOfHorizontalSteps;
    const int32 NumberOfVerticalSteps;

    int32 UnprojectedAtlasWidth;
    int32 UnprojectedAtlasHeight;

    const int32 SphericalAtlasWidth;
    const int32 SphericalAtlasHeight;

    int32 CurrentStep;
	int32 TotalSteps;

    TArray<FLinearColor> UnprojectedLeftEyeAtlas;
    TArray<FLinearColor> UnprojectedRightEyeAtlas;

    const bool bForceAlpha;
    const bool bEnableBilerp;
    const int32 SSMethod;
    const bool bOverrideInitialYaw;
    const float ForcedInitialYaw;
    const FString OutputDir;
	bool ErrorFound;
	// UseCameraRotation will be used for gathering selected rotational axis of user's camera.
	const int32 UseCameraRotation;

    bool dbgMatchCaptureSliceFovToAtlasSliceFov;
    bool dbgDisableOffsetRotation;
	FString FrameDescriptors;

	const int32 OutputBitDepth;
	const bool bOutputSceneDepth;
	const bool bOutputFinalColor;
	const bool bOutputWorldNormal;
	const bool bOutputRoughness;
	const bool bOutputMetallic;
	const bool bOutputBaseColor;
	const bool bOutputAmbientOcclusion;

	// output in mono (1 eye)
	const bool bMonoscopicMode;

    FStereoCaptureDoneDelegate StereoCaptureDoneDelegate;
};
