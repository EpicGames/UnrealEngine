// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoCapturePawn.h"
#include "StereoPanorama.h"
#include "StereoPanoramaManager.h"
#include "Engine/Texture2D.h"
#include "Engine/Engine.h"
#include "TextureResource.h"

void AStereoCapturePawn::UpdateStereoAtlas(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo)
{
	TSharedPtr<FStereoPanoramaManager> StereoPanoramaManager = FStereoPanoramaModule::Get();
	if (!StereoPanoramaManager->ValidateRendererState())
	{
		return;
	}

    FIntPoint atlasDimensions(FStereoPanoramaManager::StepCaptureWidth->GetInt(), FStereoPanoramaManager::StepCaptureWidth->GetInt() / 2);
    if (!LeftEyeAtlas || 
        !LeftEyeAtlas->IsValidLowLevel() ||
        LeftEyeAtlas->GetSizeX() != atlasDimensions.X ||
        LeftEyeAtlas->GetSizeY() != atlasDimensions.Y)
    {
        LeftEyeAtlas = UTexture2D::CreateTransient(atlasDimensions.X, atlasDimensions.Y, EPixelFormat::PF_B8G8R8A8);
        LeftEyeAtlas->SRGB = false;
        //LeftEyeAtlas->CompressionNone = true;
        LeftEyeAtlas->Filter = TextureFilter::TF_Trilinear;
    }
    
    if (!RightEyeAtlas ||
        !RightEyeAtlas->IsValidLowLevel() ||
        RightEyeAtlas->GetSizeX() != atlasDimensions.X ||
        RightEyeAtlas->GetSizeY() != atlasDimensions.Y)
    {
        RightEyeAtlas = UTexture2D::CreateTransient(atlasDimensions.X, atlasDimensions.Y, EPixelFormat::PF_B8G8R8A8);
        RightEyeAtlas->SRGB = false;
        //RightEyeAtlas->CompressionNone = true;
        RightEyeAtlas->Filter = TextureFilter::TF_Trilinear;
    }
    FStereoCaptureDoneDelegate CopyAtlasToTexDelegate;
    //TODO: ikrimae: When updating to 4.6, use stateful lambda instead of creating a separate member function
    CopyAtlasToTexDelegate.BindUObject(this, &AStereoCapturePawn::CopyAtlasDataToTextures);

	UWorld* World = GEngine->GetWorldFromContextObjectChecked(WorldContextObject);
    StereoPanoramaManager->PanoramicScreenshot(0, 0, CopyAtlasToTexDelegate, World);

    //TODO: ikrimae: Not cool b/c we're passing memory ownership to the LatentActionManager but at the same time we need a reference to this action to update when it's done
    //               Got to dig deeper into Unreal Engine to see proper way
    StereoCaptureDoneAction = new FStereoCaptureDoneAction(LatentInfo);


    if (World != nullptr)
    {
        FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
        if (LatentActionManager.FindExistingAction<FStereoCaptureDoneAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == NULL)
        {
            LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, StereoCaptureDoneAction);
        }
    }
}


void AStereoCapturePawn::CopyAtlasDataToTextures(const TArray<FLinearColor>& InLeftEyeAtlasData, const TArray<FLinearColor>& InRightEyeAtlasData)
{
    if (LeftEyeAtlas &&
        LeftEyeAtlas->IsValidLowLevel() &&
        RightEyeAtlas &&
        RightEyeAtlas->IsValidLowLevel())
    {
        {
            const int32 TextureDataSize = InLeftEyeAtlasData.Num() * InLeftEyeAtlasData.GetTypeSize();
            check(TextureDataSize == LeftEyeAtlas->GetSizeX() * LeftEyeAtlas->GetSizeY() * sizeof(FLinearColor) );

            FTexture2DMipMap& Mip = LeftEyeAtlas->GetPlatformData()->Mips[0];
            void* Data = Mip.BulkData.Lock(LOCK_READ_WRITE);
            FMemory::Memcpy(Data, InLeftEyeAtlasData.GetData(), TextureDataSize);
            Mip.BulkData.Unlock();
            LeftEyeAtlas->UpdateResource();
        }

        {
            const int32 TextureDataSize = InRightEyeAtlasData.Num() * InRightEyeAtlasData.GetTypeSize();
            check(TextureDataSize == RightEyeAtlas->GetSizeX() * RightEyeAtlas->GetSizeY() * sizeof(FLinearColor) );

            FTexture2DMipMap& Mip = RightEyeAtlas->GetPlatformData()->Mips[0];
            void* Data = Mip.BulkData.Lock(LOCK_READ_WRITE);
            FMemory::Memcpy(Data, InRightEyeAtlasData.GetData(), TextureDataSize);

            Mip.BulkData.Unlock();
            RightEyeAtlas->UpdateResource();
        }
    }

    if (StereoCaptureDoneAction)
    {
        StereoCaptureDoneAction->IsStereoCaptureDone = true;
    }
}
