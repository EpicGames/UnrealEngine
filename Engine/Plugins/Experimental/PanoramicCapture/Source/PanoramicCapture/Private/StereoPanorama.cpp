// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoPanorama.h"
#include "StereoPanoramaManager.h"
#include "Modules/ModuleManager.h"


TSharedPtr<FStereoPanoramaManager> StereoPanoramaManager;


void FStereoPanoramaModule::StartupModule()
{
	StereoPanoramaManager = MakeShareable(new FStereoPanoramaManager());
}


void FStereoPanoramaModule::ShutdownModule()
{
	if (StereoPanoramaManager.IsValid())
	{
		StereoPanoramaManager.Reset();
	}
}


TSharedPtr<FStereoPanoramaManager> FStereoPanoramaModule::Get()
{
	check(StereoPanoramaManager.IsValid());
	return StereoPanoramaManager;
}


IMPLEMENT_MODULE(FStereoPanoramaModule, PanoramicCapture)
