// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoPanoramaManager.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/World.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "LogStereoPanorama"

//Slice Controls
IConsoleVariable* FStereoPanoramaManager::HorizontalAngularIncrement = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.HorizontalAngularIncrement"), 1.0f, TEXT("The number of degrees per horizontal step. Must be a factor of 360."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::VerticalAngularIncrement   = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.VerticalAngularIncrement"), 90.0f, TEXT("The number of degrees per vertical step. Must be a factor of 180."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::CaptureHorizontalFOV       = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.CaptureHorizontalFOV"), 90.0f, TEXT("Horizontal FOV for scene capture component. Must be larger than SP.HorizontalAngularIncrement"), ECVF_Default);

//Atlas Controls
IConsoleVariable* FStereoPanoramaManager::StepCaptureWidth           = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.StepCaptureWidth"), 4096, TEXT("The final spherical atlas width"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::EyeSeparation              = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.EyeSeparation"), 6.4f, TEXT("The separation of the stereo cameras"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::ForceAlpha                 = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.ForceAlpha"), false, TEXT("Force the alpha value to completely opaque"), ECVF_Default);

//Sampling Controls
IConsoleVariable* FStereoPanoramaManager::CaptureSlicePixelWidth     = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.CaptureSlicePixelWidth"), 2048, TEXT(" Capture Slice Pixel Dimension"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::EnableBilerp               = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.EnableBilerp"), true, TEXT("0 - No Filtering 1- Bilinear Filter slice samples"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::SuperSamplingMethod        = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.SuperSamplingMethod"), 1, TEXT(" 0 - No Supersampling, 1 - Rotated Grid SS"), ECVF_Default);

//Debug Controls
//IConsoleVariable* FStereoPanoramaManager::ConcurrentCaptures         = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.ConcurrentCaptures"), 15, TEXT("The number of scene captures to capture at the same time"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::ConcurrentCaptures         = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.ConcurrentCaptures"), 30, TEXT("The number of scene captures to capture at the same time"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::GenerateDebugImages        = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.GenerateDebugImages"), 0, TEXT("0 - No Debug Images\n1 - Save out each strip as it is generated\n2 - Save each entire slice"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputDir                  = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputDir"), TEXT(""), TEXT("Output directory"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::ShouldOverrideInitialYaw   = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.ShouldOverrideInitialYaw"), true, TEXT("Override Initial Camera Yaw. Set to true if you don't want to use PlayerController View Dir"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::ForcedInitialYaw           = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.ForcedInitialYaw"), 90.0f, TEXT("Yaw value for initial Camera view direction. Set ShouldOverrideInitialYaw to true to use this value"), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::FadeStereoToZeroAtSides    = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.FadeStereoToZeroAtSides"), false, TEXT("Fade stereo effect between left/right eye to zero at 90 degrees."), ECVF_Default);

//Output Controls
IConsoleVariable* FStereoPanoramaManager::OutputFinalColor			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputFinalColor"), false, TEXT("Output final color."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputSceneDepth			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputSceneDepth"), false, TEXT("Output Scene depth pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputWorldNormal			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputWorldNormal"), false, TEXT("Output World Normal pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputRoughness			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputRoughness"), false, TEXT("Output Roughness pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputMetalic				= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputMetalic"), false, TEXT("Output Metalic pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputBaseColor			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputBaseColor"), false, TEXT("Output BaseColor pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputAmbientOcclusion	= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputAmbientOcclusion"), false, TEXT("Output AO pass."), ECVF_Default);
IConsoleVariable* FStereoPanoramaManager::OutputBitDepth			= IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.OutputBitDepth"), 32, TEXT("Output bitdepth: 8 or 32"), ECVF_Default);

// Monoscpic mode
IConsoleVariable* FStereoPanoramaManager::MonoscopicMode = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.Monoscopic"), false, TEXT("Output in monoscopic."), ECVF_Default);

//This will use camera's rotation. Use param by adding desired axis. Pitch=1 Yaw=2 Roll=4, All axis is 7 (=1+2+4)
IConsoleVariable* FStereoPanoramaManager::UseCameraRotation = IConsoleManager::Get().RegisterConsoleVariable(TEXT("SP.UseCameraRotation"), 0, TEXT("Allow each axis to follow camera rotation. Pitch=1 Yaw=2 Roll=4, All axis is 7 (=1+2+4) "), ECVF_Default);

bool FStereoPanoramaManager::ValidateRendererState() const
{
	static const auto InstancedStereoCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.InstancedStereo"));
	const bool bIsInstancedStereoEnabled = (InstancedStereoCVar && InstancedStereoCVar->GetValueOnAnyThread() != 0);

	if (bIsInstancedStereoEnabled)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Error, LOCTEXT("ValidateRendererState", "Panoramic capture not supported with instanced stereo rendering enabled."));
		return false;
	}

	return true;
}

bool FStereoPanoramaManager::CheckPreviousJobState() const
{
	if (SceneCapturer)
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Error, LOCTEXT("CheckPreviousJobState", "Previous Job is not finished."));
		return true;
	}
	return false;
	
}

void FStereoPanoramaManager::PanoramicScreenshot(const TArray<FString>& Args, UWorld* World)
{

	if (CheckPreviousJobState())
	{
		return;
	}

	if (!ValidateRendererState())
	{
		return;
	}

    FStereoCaptureDoneDelegate EmptyDelegate;
    PanoramicScreenshot(0, 0, EmptyDelegate, World);
}


void FStereoPanoramaManager::PanoramicScreenshot(const int32 InStartFrame, const int32 InEndFrame, FStereoCaptureDoneDelegate& InStereoCaptureDoneDelegate, UWorld* World)
{

	if (CheckPreviousJobState())
	{
		return;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorDelegates::EndPIE.AddRaw(this, &FStereoPanoramaManager::EndPIE);
	}
#endif

	// Construct a capturer that has stereo USceneCaptureComponent2D components
    SceneCapturer = NewObject<USceneCapturer>(World);
	SceneCapturer->AddToRoot();

	// Rotation is ignored; always start from a yaw of zero
	SceneCapturer->SetInitialState(InStartFrame, InEndFrame, InStereoCaptureDoneDelegate);
}

#if WITH_EDITOR
void FStereoPanoramaManager::EndPIE(bool bIsSimulating)
{
	Cleanup();
}
#endif

void FStereoPanoramaManager::Cleanup()
{
	if (SceneCapturer != nullptr)
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			FEditorDelegates::EndPIE.RemoveAll(this);
		}
#endif
		// Stop Tick at the time of cleanup.
		SceneCapturer->bIsTicking = false;

		SceneCapturer->Reset();
		SceneCapturer->RemoveFromRoot();

		// Let GC handle the deletion
		SceneCapturer = nullptr;
	}
}


void FStereoPanoramaManager::PanoramicMovie(const TArray<FString>& Args, UWorld* World)
{
	if (CheckPreviousJobState())
	{
		return;
	}
	
	if (!ValidateRendererState())
	{
		return;
	}
	
    int32 StartFrame = 0;
    int32 EndFrame   = 0;

	if (Args.Num() == 1)
	{
        StartFrame = 0;
        EndFrame   = FCString::Atoi(*Args[0]) - 1;    //Frame Range is inclusive so -1 to find the last frame
	}
    else if (Args.Num() == 2)
    {
        StartFrame = FCString::Atoi(*Args[0]);
        EndFrame = FCString::Atoi(*Args[1]);
        EndFrame = FMath::Max(StartFrame, EndFrame);
    }

    FStereoCaptureDoneDelegate EmptyDelegate;
    PanoramicScreenshot(StartFrame, EndFrame, EmptyDelegate, World);
}


void FStereoPanoramaManager::PanoramicQuality(const TArray<FString>& Args)
{
	if (Args.Contains(TEXT("preview")))
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("QualityInfo1", " ... setting 'preview' quality"));
		FStereoPanoramaManager::HorizontalAngularIncrement->Set(TEXT("5"));
		FStereoPanoramaManager::VerticalAngularIncrement->Set(TEXT("60"));
		FStereoPanoramaManager::CaptureHorizontalFOV->Set(TEXT("60"));
		FStereoPanoramaManager::StepCaptureWidth->Set(TEXT("720"));
	}
	else if (Args.Contains(TEXT("average")))
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("QualityInfo2", " ... setting 'average' quality"));
		FStereoPanoramaManager::HorizontalAngularIncrement->Set(TEXT("2"));
		FStereoPanoramaManager::VerticalAngularIncrement->Set(TEXT("30"));
		FStereoPanoramaManager::CaptureHorizontalFOV->Set(TEXT("30"));
		FStereoPanoramaManager::StepCaptureWidth->Set(TEXT("1440"));
	}
	else if (Args.Contains(TEXT("improved")))
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Info, LOCTEXT("QualityInfo3", " ... setting 'improved' quality"));
		FStereoPanoramaManager::HorizontalAngularIncrement->Set(TEXT("0.5"));
		FStereoPanoramaManager::VerticalAngularIncrement->Set(TEXT("22.5"));
		FStereoPanoramaManager::CaptureHorizontalFOV->Set(TEXT("22.5"));
		FStereoPanoramaManager::StepCaptureWidth->Set(TEXT("1440"));
	}
	else
	{
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, LOCTEXT("QualityWarning1", "No quality setting found; options are 'preview | average | improved'"));
	}
}


void FStereoPanoramaManager::PanoramicTogglePause(const TArray<FString>& Args)
{
	UWorld* CurrentWorld = nullptr;
	if (SceneCapturer)
	{
		CurrentWorld = SceneCapturer->GetWorld();
	}

    auto CapturePlayerController = UGameplayStatics::GetPlayerController(CurrentWorld, 0);
    auto CaptureGameMode = UGameplayStatics::GetGameMode(CurrentWorld);

    if (CaptureGameMode == nullptr || CapturePlayerController == nullptr || CurrentWorld == nullptr)
    {
		FMessageLog(StereoPanoramaLogName).Message(EMessageSeverity::Warning, LOCTEXT("QualityWarning2", "Missing GameMode or PlayerController"));
        return;
    }

	if (CurrentWorld->IsPaused())
	{
		CaptureGameMode->ClearPause();
	}
    else
    {
        CaptureGameMode->SetPause(CapturePlayerController);
    }
}


#undef LOCTEXT_NAMESPACE