// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneCapturer.h"

class FStereoPanoramaManager
{
public:

	FStereoPanoramaManager()
		: SceneCapturer(nullptr)
		, PanoramicScreenshotCommand(
			TEXT("SP.PanoramicScreenshot"),
			*NSLOCTEXT("StereoPanorama", "CommandText_ScreenShot", "Takes a panoramic screenshot").ToString(),
			FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FStereoPanoramaManager::PanoramicScreenshot))
		, PanoramicMovieCommand(
			TEXT("SP.PanoramicMovie"),
			*NSLOCTEXT("StereoPanorama", "CommandText_MovieCapture", "Takes a sequence of panoramic screenshots").ToString(),
			FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FStereoPanoramaManager::PanoramicMovie))
		, PanoramicQualityCommand(
			TEXT("SP.PanoramicQuality"),
			*NSLOCTEXT("StereoPanorama", "CommandText_Quality", "Sets the quality of the panoramic screenshot to 'preview | average | improved'").ToString(),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStereoPanoramaManager::PanoramicQuality))
		, PanoramicPauseCommand(
            TEXT("SP.TogglePause"),
			*NSLOCTEXT("StereoPanorama", "CommandText_PauseGame", "Toggles Pausing/Unpausing of the game through StereoPanorama Plugin").ToString(),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStereoPanoramaManager::PanoramicTogglePause))
	{ }

public:

	void Cleanup();
	bool ValidateRendererState() const;

	// To check previous job is finished.
	bool CheckPreviousJobState() const;

	void PanoramicScreenshot(const TArray<FString>& Args, UWorld* World);
	void PanoramicScreenshot(const int32 InStartFrame, const int32 InEndFrame, FStereoCaptureDoneDelegate& InStereoCaptureDoneDelegate, UWorld* World);
	void PanoramicMovie(const TArray<FString>& Args, UWorld* World);
	void PanoramicQuality(const TArray<FString>& Args);
	void PanoramicTogglePause(const TArray<FString>& Args);

public:

	static IConsoleVariable* HorizontalAngularIncrement;
	static IConsoleVariable* VerticalAngularIncrement;
	static IConsoleVariable* StepCaptureWidth;
	static IConsoleVariable* EyeSeparation;
	static IConsoleVariable* ForceAlpha;
	static IConsoleVariable* GenerateDebugImages;
	static IConsoleVariable* ConcurrentCaptures;
	static IConsoleVariable* CaptureHorizontalFOV;
	static IConsoleVariable* CaptureSlicePixelWidth;
	static IConsoleVariable* EnableBilerp;
	static IConsoleVariable* SuperSamplingMethod;
	static IConsoleVariable* OutputDir;
	static IConsoleVariable* ShouldOverrideInitialYaw;
	static IConsoleVariable* ForcedInitialYaw;
	static IConsoleVariable* FadeStereoToZeroAtSides;

	static IConsoleVariable* OutputFinalColor;
	static IConsoleVariable* OutputSceneDepth;
	static IConsoleVariable* OutputWorldNormal;
	static IConsoleVariable* OutputRoughness;
	static IConsoleVariable* OutputMetalic;
	static IConsoleVariable* OutputBaseColor;
	static IConsoleVariable* OutputAmbientOcclusion;
	static IConsoleVariable* OutputBitDepth;

	// Output in mono
	static IConsoleVariable* MonoscopicMode;

	/** This will use camera's rotation. Use param by adding desired axis. Pitch=1 Yaw=2 Roll=4, All axis is 7 (=1+2+4) */
	static IConsoleVariable* UseCameraRotation;

private:

	/** The scene capturer object. */
	class USceneCapturer* SceneCapturer;

private:

	void EndPIE(bool bIsSimulating);

	FAutoConsoleCommandWithWorldAndArgs PanoramicScreenshotCommand;
	FAutoConsoleCommandWithWorldAndArgs PanoramicMovieCommand;
	FAutoConsoleCommand PanoramicQualityCommand;
	FAutoConsoleCommand PanoramicPauseCommand;
};
