// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoStaticMeshComponent.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "Engine/Engine.h"
#include "StereoRendering.h"

class FStereoStaticMeshSceneProxy final
	: public FStaticMeshSceneProxy
{
    ESPStereoCameraLayer EyeToRender;

public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

    FStereoStaticMeshSceneProxy(UStereoStaticMeshComponent* Component) :
        FStaticMeshSceneProxy(Component, false)
    {
        EyeToRender = Component->EyeToRender;
    }

    virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
    {
        FPrimitiveViewRelevance viewRelevance = FStaticMeshSceneProxy::GetViewRelevance(View);
        bool bVisible = true;

		if (IStereoRendering::IsASecondaryView(*View))
		{
			if ((EyeToRender != ESPStereoCameraLayer::RightEye) && (EyeToRender != ESPStereoCameraLayer::BothEyes))
			{
				bVisible = false;
			}
		}
		else if (IStereoRendering::IsAPrimaryView(*View))
		{
			if ((EyeToRender != ESPStereoCameraLayer::LeftEye) && (EyeToRender != ESPStereoCameraLayer::BothEyes))
			{
				bVisible = false;
			}
		}

        viewRelevance.bDrawRelevance &= bVisible;

        return viewRelevance;

    }
};


FPrimitiveSceneProxy* UStereoStaticMeshComponent::CreateSceneProxy()
{
    if ((GetStaticMesh() == nullptr) ||
		(GetStaticMesh()->GetRenderData() == nullptr) ||
		(GetStaticMesh()->GetRenderData()->LODResources.Num() == 0) ||
		(GetStaticMesh()->GetRenderData()->LODResources[GetStaticMesh()->GetMinLODIdx()].VertexBuffers.PositionVertexBuffer.GetNumVertices() == 0))
    {
        return nullptr;
    }

    FPrimitiveSceneProxy* Proxy = ::new FStereoStaticMeshSceneProxy(this);

	return Proxy;
}
