// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/StaticMeshComponent.h"
#include "StereoStaticMeshComponent.generated.h"


UENUM(BlueprintType)
enum class ESPStereoCameraLayer : uint8
{
    LeftEye,
    RightEye,
    BothEyes
};


/**
 * 
 */
UCLASS(ClassGroup = (Rendering, Common), hidecategories = (Object, Activation, "Components|Activation"), ShowCategories = (Mobility), editinlinenew, meta = (BlueprintSpawnableComponent))
class UStereoStaticMeshComponent
	: public UStaticMeshComponent
{
    GENERATED_BODY()

public:

    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sprite)
    ESPStereoCameraLayer EyeToRender;

    virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	
};
