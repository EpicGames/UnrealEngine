// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/DefaultPawn.h"
#include "LatentActions.h"
#include "Engine/LatentActionManager.h"
#include "StereoCapturePawn.generated.h"


class FStereoCaptureDoneAction
	: public FPendingLatentAction
{
public:

    FName ExecutionFunction;
    int32 OutputLink;
    FWeakObjectPtr CallbackTarget;
    bool IsStereoCaptureDone;

public:

    FStereoCaptureDoneAction(const FLatentActionInfo& LatentInfo)
        : ExecutionFunction(LatentInfo.ExecutionFunction)
        , OutputLink(LatentInfo.Linkage)
        , CallbackTarget(LatentInfo.CallbackTarget)
        , IsStereoCaptureDone(false)
    {
    }

    virtual void UpdateOperation(FLatentResponse& Response)
    {
        Response.FinishAndTriggerIf(IsStereoCaptureDone, ExecutionFunction, OutputLink, CallbackTarget);
    }

#if WITH_EDITOR
    // Returns a human readable description of the latent operation's current state
    virtual FString GetDescription() const override
    {
		return FText::Format(NSLOCTEXT("StereoCaptureDoneAction", "IsStereoCaptureDoneFmt", "Is Stereo Capture Done: {0}"), IsStereoCaptureDone).ToString();
    }
#endif
};


/**
 * 
 */
UCLASS(config = Game, Blueprintable, BlueprintType)
class AStereoCapturePawn
	: public ADefaultPawn
{
    GENERATED_BODY()

public:

    UFUNCTION(BlueprintCallable, Category = "Pawn", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo"))
    void UpdateStereoAtlas(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo);

    UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Pawn)
    TObjectPtr<UTexture2D> LeftEyeAtlas;

    UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Pawn)
    TObjectPtr<UTexture2D> RightEyeAtlas;


    FStereoCaptureDoneAction* StereoCaptureDoneAction;

    void CopyAtlasDataToTextures(const TArray<FLinearColor>& LeftEyeAtlasData, const TArray<FLinearColor>& RightEyeAtlasData);
};
