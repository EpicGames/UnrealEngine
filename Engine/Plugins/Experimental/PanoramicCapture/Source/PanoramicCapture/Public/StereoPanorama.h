// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FStereoPanoramaManager;

/**
 * Implements the StereoPanorama module.
 */
class FStereoPanoramaModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:

	static TSharedPtr<FStereoPanoramaManager> Get();
};
